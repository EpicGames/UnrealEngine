// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationWorker : ModuleRules
	{
		public AutomationWorker(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			); 
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AutomationMessages",
					"AutomationTest",
					"CoreUObject",
                    "Analytics",
    				"AnalyticsET",
					"Json",
					"JsonUtilities"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"MessagingCommon",
				}
			);

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.Add("Engine");
				PrivateDependencyModuleNames.Add("RHI");
			}
		}
	}
}
