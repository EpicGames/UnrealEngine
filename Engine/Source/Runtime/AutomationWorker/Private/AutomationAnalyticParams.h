// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


namespace EAutomationEventName
{
	enum Type
	{
		FPSCapture,
		AutomationTestResults,

		NUM_ENGINE_EVENT_NAMES
	};
}


namespace EAutomationAnalyticParam
{
	enum Type
	{
		MapName,
		TimeStamp,
		Platform,
		Spec,
		CL,
		FPS,
		BuildConfiguration,
		AverageFrameTime,
		AverageGameThreadTime,
		AverageRenderThreadTime,
		AverageGPUTime,
		PercentOfFramesAtLeast30FPS,
		PercentOfFramesAtLeast60FPS,

		TestName,
		BeautifiedName,
		ExecutionCount,
		IsSuccess,
		Duration,
		ErrorCount,
		WarningCount,

		NUM_ENGINE_PARAMS
	};
}
