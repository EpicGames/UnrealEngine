// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationAnalytics.h"
#include "AutomationWorkerPrivate.h"

#include "AnalyticsET.h"
#include "AnalyticsEventAttribute.h"
#include "AutomationWorkerMessages.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "IAnalyticsProviderET.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/EngineVersion.h"

#include "IAutomationWorkerModule.h"


DEFINE_LOG_CATEGORY(LogAutomationAnalytics);


/* Static initialization
 *****************************************************************************/

bool FAutomationAnalytics::bIsInitialized;
TSharedPtr<IAnalyticsProviderET> FAutomationAnalytics::Analytics;
TArray<FString> FAutomationAnalytics::AutomationEventNames;
TArray<FString> FAutomationAnalytics::AutomationParamNames;
FString FAutomationAnalytics::MachineSpec;


/* FAutomationAnalytics static functions
 *****************************************************************************/

IAnalyticsProvider& FAutomationAnalytics::GetProvider()
{
	checkf(bIsInitialized && IsAvailable(), TEXT("FAutomationAnalytics::GetProvider called outside of Initialize/Shutdown."));

	return *Analytics.Get();
}


void FAutomationAnalytics::Initialize()
{
	checkf(!bIsInitialized, TEXT("FAutomationAnalytics::Initialize called more than once."));

	//Set the config for Analytics
	Analytics = FAnalyticsET::Get().CreateAnalyticsProvider(FAnalyticsET::Config(FString::Printf(TEXT("AutomationAnalytics.%s"), GInternalProjectName), TEXT("https://datarouter.ol.epicgames.com/")));
	if (Analytics.IsValid())
	{
		Analytics->SetUserID(FString::Printf(TEXT("%s|%s|%s"), *FPlatformMisc::GetLoginId(), *FPlatformMisc::GetEpicAccountId(), *FPlatformMisc::GetOperatingSystemId()));
		Analytics->StartSession();

		MachineSpec = FParse::Param(FCommandLine::Get(), TEXT("60hzmin")) 
			? TEXT("60hzmin")
			: FParse::Param(FCommandLine::Get(), TEXT("30hzmin")) 
			? TEXT("30hzmin") 
			: TEXT("");

		bIsInitialized = true;
		InititalizeAnalyticParameterNames();
	}
}


bool FAutomationAnalytics::IsAvailable()
{
	return Analytics.IsValid();
}


void FAutomationAnalytics::Shutdown()
{
	Analytics.Reset();
	bIsInitialized = false;
}


FString FAutomationAnalytics::GetAutomationEventName(EAutomationEventName::Type InEventName)
{
	check(InEventName < AutomationEventNames.Num());
	return AutomationEventNames[InEventName];
}


FString FAutomationAnalytics::GetAutomationParamName(EAutomationAnalyticParam::Type InEngineParam)
{
	check(InEngineParam < AutomationParamNames.Num());
	return AutomationParamNames[InEngineParam];
}


void FAutomationAnalytics::InititalizeAnalyticParameterNames()
{
	// Engine Event Names
	AutomationEventNames.Reserve(EAutomationEventName::NUM_ENGINE_EVENT_NAMES);
	AutomationEventNames.SetNum(EAutomationEventName::NUM_ENGINE_EVENT_NAMES);

	AutomationEventNames[EAutomationEventName::FPSCapture] = TEXT("FPSCapture");
	AutomationEventNames[EAutomationEventName::AutomationTestResults] = TEXT("AutomationTest");

	for (int32 EventIndex = 0; EventIndex < EAutomationEventName::NUM_ENGINE_EVENT_NAMES; EventIndex++)
	{
		if (AutomationEventNames[EventIndex].IsEmpty())
		{
			UE_LOGF(LogAutomationAnalytics, Error, "Engine Events missing a event: %d!", EventIndex);
		}
	}

	// Engine Event Params
	AutomationParamNames.Reserve(EAutomationAnalyticParam::NUM_ENGINE_PARAMS);
	AutomationParamNames.SetNum(EAutomationAnalyticParam::NUM_ENGINE_PARAMS);

	AutomationParamNames[EAutomationAnalyticParam::MapName] = TEXT("MapName");
	AutomationParamNames[EAutomationAnalyticParam::TimeStamp] = TEXT("TimeStamp");
	AutomationParamNames[EAutomationAnalyticParam::Platform] = TEXT("Platform");
	AutomationParamNames[EAutomationAnalyticParam::Spec] = TEXT("Spec");
	AutomationParamNames[EAutomationAnalyticParam::CL] = TEXT("CL");
	AutomationParamNames[EAutomationAnalyticParam::FPS] = TEXT("FPS");
	AutomationParamNames[EAutomationAnalyticParam::BuildConfiguration] = TEXT("BuildConfiguration");
	AutomationParamNames[EAutomationAnalyticParam::AverageFrameTime] = TEXT("AverageFrameTime");
	AutomationParamNames[EAutomationAnalyticParam::AverageGameThreadTime] = TEXT("AverageGameThreadTime");
	AutomationParamNames[EAutomationAnalyticParam::AverageRenderThreadTime] = TEXT("AverageRenderThreadTime");
	AutomationParamNames[EAutomationAnalyticParam::AverageGPUTime] = TEXT("AverageGPUTime");
	AutomationParamNames[EAutomationAnalyticParam::PercentOfFramesAtLeast30FPS] = TEXT("PercentOfFramesAtLeast30FPS");
	AutomationParamNames[EAutomationAnalyticParam::PercentOfFramesAtLeast60FPS] = TEXT("PercentOfFramesAtLeast60FPS");

	AutomationParamNames[EAutomationAnalyticParam::TestName] = TEXT("TestName");
	AutomationParamNames[EAutomationAnalyticParam::BeautifiedName] = TEXT("BeautifiedName");
	AutomationParamNames[EAutomationAnalyticParam::ExecutionCount] = TEXT("ExecutionCount");
	AutomationParamNames[EAutomationAnalyticParam::IsSuccess] = TEXT("IsSuccess");
	AutomationParamNames[EAutomationAnalyticParam::Duration] = TEXT("Duration");
	AutomationParamNames[EAutomationAnalyticParam::ErrorCount] = TEXT("ErrorCount");
	AutomationParamNames[EAutomationAnalyticParam::WarningCount] = TEXT("WarningCount");

	for (int32 ParamIndex = 0; ParamIndex < EAutomationAnalyticParam::NUM_ENGINE_PARAMS; ParamIndex++)
	{
		if (AutomationParamNames[ParamIndex].IsEmpty())
		{
			UE_LOGF(LogAutomationAnalytics, Error, "Engine Events missing a param: %d!", ParamIndex);
		}
	}
}


void FAutomationAnalytics::SetInitialParameters(TArray<FAnalyticsEventAttribute>& ParamArray)
{
	int32 TimeStamp;
	FString PlatformName;

	TimeStamp = FApp::GetCurrentTime();
	PlatformName = FPlatformProperties::PlatformName();

	ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::TimeStamp), TimeStamp));
	ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::Platform), PlatformName));
}


void FAutomationAnalytics::FireEvent_FPSCapture(const FAutomationPerformanceSnapshot& PerfSnapshot)
{
	// @EventName FPSCapture
	//
	// @Trigger 
	//
	// @Type Static
	//
	// @EventParam TimeStamp int32 The time in seconds when the test when the capture ended
	// @EventParam Platform string The platform in which the test is run on
	// @EventParam CL string The Changelist for the build
	// @EventParam Spec string The spec of the machine running the test
	// @EventParam MapName string The map that the test was run on
	// @EventParam FPS string 
	// @EventParam BuildConfiguration string Debug/Development/Test/Shipping
	// @EventParam AverageFrameTime string Time for a frame in ms
	// @EventParam AverageGameThreadTime string Time for the game thread in ms
	// @EventParam AverageRenderThreadTime string Time for the rendering thread in ms
	// @EventParam AverageGPUTime string Time for the GPU to flush in ms
	// @EventParam PercentOfFramesAtLeast30FPS string 
	// @EventParam PercentOfFramesAtLeast60FPS string 
	//
	// @Comments

	if (Analytics.IsValid())
	{
		TArray<FAnalyticsEventAttribute> ParamArray;
		SetInitialParameters(ParamArray);
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::CL), PerfSnapshot.Changelist));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::Spec), MachineSpec));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::MapName), PerfSnapshot.MapName));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::FPS), PerfSnapshot.AverageFPS));

		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::BuildConfiguration), PerfSnapshot.BuildConfiguration));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::AverageFrameTime), PerfSnapshot.AverageFrameTime));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::AverageGameThreadTime), PerfSnapshot.AverageGameThreadTime));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::AverageRenderThreadTime), PerfSnapshot.AverageRenderThreadTime));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::AverageGPUTime), PerfSnapshot.AverageGPUTime));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::PercentOfFramesAtLeast30FPS), PerfSnapshot.PercentOfFramesAtLeast30FPS));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::PercentOfFramesAtLeast60FPS), PerfSnapshot.PercentOfFramesAtLeast60FPS));

		Analytics->RecordEvent(GetAutomationEventName(EAutomationEventName::FPSCapture), ParamArray);
	}
}


void FAutomationAnalytics::FireEvent_AutomationTestResults(const FAutomationWorkerRunTestsReply* TestResults, const FString& BeautifiedTestName)
{
	// @EventName AutomationTestResults
	//
	// @Trigger Fires when an automation test has completed
	//
	// @Type Static
	//
	// @EventParam TimeStamp int32 The time in seconds when the test ended
	// @EventParam Platform string The platform in which the test is run on
	// @EventParam CL string The Changelist for the build
	// @EventParam TestName string The execution string of the test being run
	// @EventParam BeautifiedTestName string The name displayed in the UI
	// @EventParam ExecutionCount string
	// @EventParam IsSuccess bool Where the test was a success or not
	// @EventParam Duration float The duration the test took to complete
	// @EventParam ErrorCount int32 The amount of errors during the test
	// @EventParam WarningCount int32 The amount of warnings during the test
	// @EventParam CL string The Changelist for the build
	//
	// @Comments

	if (Analytics.IsValid())
	{
		TArray<FAnalyticsEventAttribute> ParamArray;
		SetInitialParameters(ParamArray);
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::TestName), TestResults->TestName));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::BeautifiedName), BeautifiedTestName));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::ExecutionCount), TestResults->ExecutionCount));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::IsSuccess), TestResults->State == EAutomationState::Success));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::Duration), TestResults->Duration));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::ErrorCount), TestResults->ErrorTotal));
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::WarningCount), TestResults->WarningTotal));

		const int32 ChangeList = FEngineVersion::Current().GetChangelist();
		FString ChangeListString = FString::FromInt(ChangeList);
		ParamArray.Add(FAnalyticsEventAttribute(GetAutomationParamName(EAutomationAnalyticParam::CL), ChangeListString));

		Analytics->RecordEvent(GetAutomationEventName(EAutomationEventName::AutomationTestResults), ParamArray);
	}
}

bool FAutomationAnalytics::IsInitialized()
{
	return bIsInitialized;
}
