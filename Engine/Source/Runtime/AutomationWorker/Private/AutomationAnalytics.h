// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#include "AutomationAnalyticParams.h"

class IAnalyticsProvider;
class IAnalyticsProviderET;

struct FAnalyticsEventAttribute;
struct FAutomationPerformanceSnapshot;
struct FAutomationWorkerRunTestsReply;


class FAutomationAnalytics
	: FNoncopyable
{
public:

	/**
	 * Return the provider instance. Not valid outside of Initialize/Shutdown calls.
	 *
	 * Note: must check IsAvailable() first else this code will assert if the provider is not valid.
	 */
	static IAnalyticsProvider& GetProvider();

	/** Called to initialize the singleton. */
	static void Initialize();
	
	/** Helper function to determine if the provider is valid. */
	static bool IsAvailable();

	/** Called to shut down the singleton */
	static void Shutdown();

	/*
	 * Helper function to retrieve the parameter name from a given enum
	 */
	static FString GetAutomationEventName(EAutomationEventName::Type InEventName);
	static FString GetAutomationParamName(EAutomationAnalyticParam::Type InEngineParam);

	static void InititalizeAnalyticParameterNames();
	
	/** Helper to check if the Analytics provider is Initialized */
	static bool IsInitialized();

public:

	//////////////////////////////////////////////////////////////////////////
	//	Events
	//////////////////////////////////////////////////////////////////////////

	//FPSCapture Event
	static void FireEvent_FPSCapture(const FAutomationPerformanceSnapshot& PerfSnapshot);

	//AutomationTest Event
	static void FireEvent_AutomationTestResults(const FAutomationWorkerRunTestsReply* TestResults, const FString& BeautifiedTestName);

private:
	
	static void SetInitialParameters(TArray<FAnalyticsEventAttribute>& ParamArray);

	/** Array of Analytic Parameter names */
	static TArray<FString> AutomationEventNames;
	static TArray<FString> AutomationParamNames;
	static FString MachineSpec;

	static bool bIsInitialized;

	/** This allows us to "opt-out" of engine automation analytics. We still create a session, but we don't send any automation events. */
	static TSharedPtr<IAnalyticsProviderET> Analytics;
};
