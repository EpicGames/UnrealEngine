// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationWorkerModule.h"

#include "Algo/Reverse.h"
#include "AutomationAnalytics.h"
#include "AutomationWorkerMessages.h"
#include "AutomationTestExcludelist.h"
#include "HAL/FileManager.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"

#if WITH_ENGINE
	#include "Engine/Engine.h"
	#include "Engine/GameViewportClient.h"
	#include "ImageUtils.h"
	#include "Tests/AutomationCommon.h"
	#include "UnrealClient.h"
	#include "RHIFeatureLevel.h"
	#include "RHIStrings.h"
#endif

#if WITH_EDITOR
	#include "AssetRegistry/AssetRegistryModule.h"
#endif


#define LOCTEXT_NAMESPACE "AutomationTest"

DEFINE_LOG_CATEGORY_STATIC(LogAutomationWorker, Log, All);

IMPLEMENT_MODULE(FAutomationWorkerModule, AutomationWorker);

/* IModuleInterface interface
 *****************************************************************************/

void FAutomationWorkerModule::StartupModule()
{
	Initialize();

	FAutomationTestFramework::Get().PreTestingEvent.AddRaw(this, &FAutomationWorkerModule::HandlePreTestingEvent);
	FAutomationTestFramework::Get().PostTestingEvent.AddRaw(this, &FAutomationWorkerModule::HandlePostTestingEvent);
}

void FAutomationWorkerModule::ShutdownModule()
{
	FAutomationTestFramework::Get().PreTestingEvent.RemoveAll(this);
	FAutomationTestFramework::Get().PostTestingEvent.RemoveAll(this);
}

bool FAutomationWorkerModule::SupportsDynamicReloading()
{
	return true;
}


/* IAutomationWorkerModule interface
 *****************************************************************************/

void FAutomationWorkerModule::Tick()
{
	//execute latent commands from the previous frame.  Gives the rest of the engine a turn to tick before closing the test
	bool bAllLatentCommandsComplete  = ExecuteLatentCommands();
	if (bAllLatentCommandsComplete)
	{
		//if we were running the latent commands as a result of executing a network command, report that we are now done
		if (bExecutingNetworkCommandResults)
		{
			ReportNetworkCommandComplete();
			bExecutingNetworkCommandResults = false;
		}

		//if the controller has requested the next network command be executed
		if (bExecuteNextNetworkCommand)
		{
			//execute network commands if there are any queued up and our role is appropriate
			bool bAllNetworkCommandsComplete = ExecuteNetworkCommands();
			if (bAllNetworkCommandsComplete)
			{
				ReportTestComplete();
			}

			//we've now executed a network command which may have enqueued further latent actions
			bExecutingNetworkCommandResults = true;

			//do not execute anything else until expressly told to by the controller
			bExecuteNextNetworkCommand = false;
		}
	}

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->ProcessInbox();
	}
}


/* ISessionManager implementation
 *****************************************************************************/

bool FAutomationWorkerModule::ExecuteLatentCommands()
{
	bool bAllLatentCommandsComplete = false;
	
	if (GIsAutomationTesting)
	{
		// Ensure that latent automation commands have time to execute
		bAllLatentCommandsComplete = FAutomationTestFramework::Get().ExecuteLatentCommands();
	}
	
	return bAllLatentCommandsComplete;
}


bool FAutomationWorkerModule::ExecuteNetworkCommands()
{
	bool bAllLatentCommandsComplete = false;
	
	if (GIsAutomationTesting)
	{
		// Ensure that latent automation commands have time to execute
		bAllLatentCommandsComplete = FAutomationTestFramework::Get().ExecuteNetworkCommands();
	}

	return bAllLatentCommandsComplete;
}


void FAutomationWorkerModule::Initialize()
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		// initialize messaging
		MessageEndpoint = FMessageEndpoint::Builder("FAutomationWorkerModule")
			.Handling<FAutomationWorkerFindWorkers>(this, &FAutomationWorkerModule::HandleFindWorkersMessage)
			.Handling<FAutomationWorkerNextNetworkCommandReply>(this, &FAutomationWorkerModule::HandleNextNetworkCommandReplyMessage)
			.Handling<FAutomationWorkerPing>(this, &FAutomationWorkerModule::HandlePingMessage)
			.Handling<FAutomationWorkerStartTestSession>(this, &FAutomationWorkerModule::HandleStartTestSession)
			.Handling<FAutomationWorkerStopTestSession>(this, &FAutomationWorkerModule::HandleStopTestSession)
			.Handling<FAutomationWorkerRequestTests>(this, &FAutomationWorkerModule::HandleRequestTestsMessage)
			.Handling<FAutomationWorkerRunTests>(this, &FAutomationWorkerModule::HandleRunTestsMessage)
			.Handling<FAutomationWorkerImageComparisonResults>(this, &FAutomationWorkerModule::HandleScreenShotCompared)
			.Handling<FAutomationWorkerTestDataResponse>(this, &FAutomationWorkerModule::HandleTestDataRetrieved)
			.Handling<FAutomationWorkerStopTests>(this, &FAutomationWorkerModule::HandleStopTestsMessage)
			.WithInbox();

		if (MessageEndpoint.IsValid())
		{
			MessageEndpoint->Subscribe<FAutomationWorkerFindWorkers>();
		}

		bExecuteNextNetworkCommand = true;		
	}
	else
	{
		bExecuteNextNetworkCommand = false;
	}
	ExecutionCount = INDEX_NONE;
	bExecutingNetworkCommandResults = false;
	bSendAnalytics = false;

	FParse::Value(FCommandLine::Get(), TEXT("-DeviceTag="), DeviceTag);
}

void FAutomationWorkerModule::ReportNetworkCommandComplete()
{
	if (GIsAutomationTesting)
	{
		SendMessage(
			FMessageEndpoint::MakeMessage<FAutomationWorkerRequestNextNetworkCommand>(ExecutionCount),
			FAutomationWorkerRequestNextNetworkCommand::StaticStruct(),
			TestRequesterAddress);
		if (StopTestEvent.IsBound())
		{
			// this is a local test; the message to continue will never arrive, so lets not wait for it
			bExecuteNextNetworkCommand = true;
		}
	}
}

void FAutomationWorkerModule::ReportTestComplete()
{
	if (GIsAutomationTesting)
	{
		//see if there are any more network commands left to execute
		bool bAllLatentCommandsComplete = FAutomationTestFramework::Get().ExecuteLatentCommands();

		//structure to track error/warning/log messages
		FAutomationTestExecutionInfo ExecutionInfo;

		bool bSuccess = FAutomationTestFramework::Get().StopTest(ExecutionInfo);

		if (StopTestEvent.IsBound())
		{
			StopTestEvent.Execute(bSuccess, TestName, ExecutionInfo);
		}
		else
		{
			// send the results to the controller
			FAutomationWorkerRunTestsReply* Message = FMessageEndpoint::MakeMessage<FAutomationWorkerRunTestsReply>();

			Message->TestName = TestName;
			Message->ExecutionCount = ExecutionCount;
			Message->State = bSuccess ? EAutomationState::Success : EAutomationState::Fail;
			Message->Duration = ExecutionInfo.Duration;
			// Prune log entries if it is a success to reduce foot print.
			Message->Entries = bPruneLogsOnSuccess && bSuccess ?
				ExecutionInfo.GetEntries().FilterByPredicate([](const FAutomationExecutionEntry& Entry) { return Entry.Event.Context != TEXT("log") || Entry.Event.Type != EAutomationEventType::Info ; }) : ExecutionInfo.GetEntries();
			Message->WarningTotal = ExecutionInfo.GetWarningTotal();
			Message->ErrorTotal = ExecutionInfo.GetErrorTotal();

			// sending though the endpoint will free Message memory, so analytics need to be sent first
			if (bSendAnalytics)
			{
				if (!FAutomationAnalytics::IsInitialized())
				{
					FAutomationAnalytics::Initialize();
				}
				FAutomationAnalytics::FireEvent_AutomationTestResults(Message, BeautifiedTestName);
				SendAnalyticsEvents(ExecutionInfo.AnalyticsItems);
			}

			if (ExecutionInfo.TelemetryItems.Num() > 0)
			{
				HandleTelemetryData(ExecutionInfo.TelemetryStorage, FullTestPath, ExecutionInfo.TelemetryItems);
			}

			SendMessage(Message, Message->StaticStruct(), TestRequesterAddress);
		}


		// reset local state
		TestRequesterAddress.Invalidate();
		ExecutionCount = INDEX_NONE;
		TestName.Empty();
		FullTestPath.Empty();
		BeautifiedTestName.Empty();
		StopTestEvent.Unbind();
	}
}


void FAutomationWorkerModule::SendTests( const FMessageAddress& ControllerAddress )
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Worker"));
	FAutomationWorkerRequestTestsReplyComplete* Reply = FMessageEndpoint::MakeMessage<FAutomationWorkerRequestTestsReplyComplete>();
	for( int32 TestIndex = 0; TestIndex < TestInfo.Num(); TestIndex++ )
	{
		Reply->Tests.Emplace(FAutomationWorkerSingleTestReply(TestInfo[TestIndex]));
	}

	UE_LOGF(LogAutomationWorker, Log, "Set %d tests to %ls", TestInfo.Num(), *ControllerAddress.ToString());

	SendMessage(Reply, Reply->StaticStruct(), ControllerAddress);
}

void FAutomationWorkerModule::SendMessage(FAutomationWorkerMessageBase* Message, UScriptStruct* TypeInfo, const FMessageAddress& ControllerAddress)
{
	check(nullptr != Message);

	Message->InstanceId = FApp::GetInstanceId();
	MessageEndpoint->Send(
		Message,
		TypeInfo,
		EMessageFlags::None,
		nullptr,
		TArrayBuilder<FMessageAddress>().Add(ControllerAddress),
		FTimespan::Zero(),
		FDateTime::MaxValue());
}

/* FAutomationWorkerModule callbacks
 *****************************************************************************/

void FAutomationWorkerModule::HandleFindWorkersMessage(const FAutomationWorkerFindWorkers& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationWorker, Log, "Received FindWorkersMessage from %ls", *Context->GetSender().ToString());

	// Set the Instance name to be the same as the session browser. This information should be shared at some point
	if ((Message.SessionId == FApp::GetSessionId()) && (Message.Changelist == 10000))
	{
#if WITH_EDITOR
		//If the asset registry is loading assets then we'll wait for it to stop before running our automation tests.
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			if (!AssetRegistryModule.Get().OnFilesLoaded().IsBoundToObject(this))
			{
				AssetRegistryModule.Get().OnFilesLoaded().AddLambda([this, Context] {
						SendWorkerFound(Context->GetSender());
					}
				);
				GLog->Logf(ELogVerbosity::Log, TEXT("...Forcing Asset Registry Load For Automation"));
			}
		}
		else
#endif
		{
			//If the registry is not loading then we'll just go ahead and run our tests.
			SendWorkerFound(Context->GetSender());
		}
	}
}


void FAutomationWorkerModule::SendWorkerFound(const FMessageAddress& ControllerAddress)
{
	FAutomationWorkerFindWorkersResponse* Response = FMessageEndpoint::MakeMessage<FAutomationWorkerFindWorkersResponse>();

	FString OSMajorVersionString, OSSubVersionString;
	FPlatformMisc::GetOSVersions(OSMajorVersionString, OSSubVersionString);

	FString OSVersionString = OSMajorVersionString + TEXT(" ") + OSSubVersionString;
	FString CPUModelString = FPlatformMisc::GetCPUBrand().TrimStart();

	FString DeviceName = DeviceTag.IsEmpty() ? FPlatformProcess::ComputerName() : DeviceTag;

	Response->DeviceName = DeviceName;
	Response->InstanceName = FApp::GetInstanceName();
	Response->Platform = FPlatformProperties::PlatformName();
	Response->SessionId = FApp::GetSessionId();
	Response->OSVersionName = OSVersionString;
	Response->ModelName = FPlatformMisc::GetDefaultDeviceProfileName();
	Response->GPUName = FPlatformMisc::GetPrimaryGPUBrand();
	Response->CPUModelName = CPUModelString;
	Response->RAMInGB = FPlatformMemory::GetPhysicalGBRam();
	Response->RHIName = FApp::GetGraphicsRHI();
#if WITH_ENGINE && WITH_AUTOMATION_TESTS
	Response->RenderModeName = AutomationCommon::GetRenderDetailsString();
#else
	Response->RenderModeName = TEXT("Unknown");
#endif

	SendMessage(Response, Response->StaticStruct(), ControllerAddress);
}


void FAutomationWorkerModule::HandleNextNetworkCommandReplyMessage( const FAutomationWorkerNextNetworkCommandReply& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	UE_LOGF(LogAutomationWorker, Log, "Received NextNetworkCommandReplyMessage from %ls", *Context->GetSender().ToString());

	// Allow the next command to execute
	bExecuteNextNetworkCommand = true;

	// We should never be executing sub-commands of a network command when we're waiting for a cue for the next network command
	check(bExecutingNetworkCommandResults == false);
}


void FAutomationWorkerModule::HandlePingMessage( const FAutomationWorkerPing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	SendMessage(FMessageEndpoint::MakeMessage<FAutomationWorkerPong>(), FAutomationWorkerPong::StaticStruct(), Context->GetSender());
}


void FAutomationWorkerModule::HandleStartTestSession( const FAutomationWorkerStartTestSession& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	UE_LOGF(LogAutomationWorker, Log, "Received StartTestSession from %ls", *Context->GetSender().ToString());

	FAutomationTestFramework::Get().ResetTests();
	ActiveSection.Empty();
	FAutomationTestFramework::Get().OnBeforeAllTestsEvent.Broadcast();
}

void FAutomationWorkerModule::HandleStopTestSession(const FAutomationWorkerStopTestSession& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationWorker, Log, "Received StopTestSession from %ls", *Context->GetSender().ToString());

	// Unwind Active Section
	TriggerSectionNotifications();

	FAutomationTestFramework::Get().OnAfterAllTestsEvent.Broadcast();
}


void FAutomationWorkerModule::HandleRequestTestsMessage( const FAutomationWorkerRequestTests& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	UE_LOGF(LogAutomationWorker, Log, "Received RequestTestsMessage from %ls", *Context->GetSender().ToString());

	FAutomationTestFramework::Get().LoadTestModules();
	FAutomationTestFramework::Get().SetDeveloperDirectoryIncluded(Message.DeveloperDirectoryIncluded);
	FAutomationTestFramework::Get().SetRequestedTestFilter((EAutomationTestFlags)Message.RequestedTestFlags);
	FAutomationTestFramework::Get().GetValidTestNames( TestInfo );

	SendTests(Context->GetSender());
}


void FAutomationWorkerModule::HandlePreTestingEvent()
{
#if WITH_ENGINE
	FAutomationTestFramework::Get().OnScreenshotCaptured().BindRaw(this, &FAutomationWorkerModule::HandleScreenShotCapturedWithName);
	FAutomationTestFramework::Get().OnScreenshotAndTraceCaptured().BindRaw(this, &FAutomationWorkerModule::HandleScreenShotAndTraceCapturedWithName);
	FAutomationTestFramework::Get().OnScreenshotComparisonReport.AddRaw(this, &FAutomationWorkerModule::HandleScreenShotComparisonReport);
#endif
}


void FAutomationWorkerModule::HandlePostTestingEvent()
{
#if WITH_ENGINE
	FAutomationTestFramework::Get().OnScreenshotAndTraceCaptured().Unbind();
	FAutomationTestFramework::Get().OnScreenshotCaptured().Unbind();
	FAutomationTestFramework::Get().OnScreenshotComparisonReport.RemoveAll(this);
#endif
}


void FAutomationWorkerModule::HandleScreenShotCompared(const FAutomationWorkerImageComparisonResults& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationWorker, Log, "Received ScreenShotCompared from %ls", *Context->GetSender().ToString());

	if (Message.UniqueId != ActiveScreenshotComparisonId)
	{
		UE_LOGF(LogAutomationWorker, Log, "Ignoring unexpected screenshot comparison result for %ls", *Message.ScreenshotPath);
		return;
	}

	// Image comparison finished.
	FAutomationScreenshotCompareResults CompareResults;
	CompareResults.UniqueId = Message.UniqueId;
	CompareResults.ScreenshotPath = Message.ScreenshotPath;
	CompareResults.bWasNew = Message.bNew;
	CompareResults.bWasSimilar = Message.bSimilar;
	CompareResults.MaxLocalDifference = Message.MaxLocalDifference;
	CompareResults.GlobalDifference = Message.GlobalDifference;
	CompareResults.ErrorMessage = Message.ErrorMessage;

	FAutomationTestFramework::Get().NotifyScreenshotComparisonComplete(CompareResults);
}

void FAutomationWorkerModule::HandleTestDataRetrieved(const FAutomationWorkerTestDataResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationWorker, Log, "Received TestDataRetrieved from %ls", *Context->GetSender().ToString());

	FAutomationTestFramework::Get().NotifyTestDataRetrieved(Message.bIsNew, Message.JsonData);
}

void FAutomationWorkerModule::HandlePerformanceDataRetrieved(const FAutomationWorkerPerformanceDataResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationWorker, Log, "Received PerformanceDataRetrieved from %ls", *Context->GetSender().ToString());

	FAutomationTestFramework::Get().NotifyPerformanceDataRetrieved(Message.bSuccess, Message.ErrorMessage);
}

#if WITH_ENGINE
void FAutomationWorkerModule::HandleScreenShotComparisonReport(const FAutomationScreenshotCompareResults& Results)
{
	FAutomationWorkerImageComparisonResults* Message = FMessageEndpoint::MakeMessage<FAutomationWorkerImageComparisonResults>();

	Message->ScreenshotPath = Results.ScreenshotPath;
	Message->UniqueId = Results.UniqueId;
	Message->bNew = Results.bWasNew;
	Message->bSimilar = Results.bWasSimilar;
	Message->ErrorMessage = Results.ErrorMessage;
	Message->MaxLocalDifference = Results.MaxLocalDifference;
	Message->GlobalDifference = Results.GlobalDifference;
	Message->IncomingFilePath = Results.IncomingFilePath;
	Message->ReportComparisonFilePath = Results.ReportComparisonFilePath;
	Message->ReportApprovedFilePath = Results.ReportApprovedFilePath;
	Message->ReportIncomingFilePath = Results.ReportIncomingFilePath;

	SendMessage(Message, Message->StaticStruct(), TestRequesterAddress);
}

void FAutomationWorkerModule::HandleScreenShotCapturedWithName(const TArray<FColor>& RawImageData, const FAutomationScreenshotData& Data)
{
	HandleScreenShotAndTraceCapturedWithName(RawImageData, TArray<uint8>(), Data);
}

void FAutomationWorkerModule::HandleScreenShotAndTraceCapturedWithName(const TArray<FColor>& RawImageData, const TArray<uint8>& CapturedFrameTrace, const FAutomationScreenshotData& Data)
{
#if WITH_AUTOMATION_TESTS
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/ImageCompare"));
	int32 NewHeight = Data.Height;
	int32 NewWidth = Data.Width;

	TArray64<uint8> CompressedBitmap;
	FImageUtils::PNGCompressImageArray(NewWidth, NewHeight, TArrayView64<const FColor>(RawImageData.GetData(), RawImageData.Num()), CompressedBitmap);

	FAutomationScreenshotMetadata Metadata(Data);
		
	// Send the screen shot if we have a target
	if (TestRequesterAddress.IsValid())
	{
		FAutomationWorkerScreenImage* Message = FMessageEndpoint::MakeMessage<FAutomationWorkerScreenImage>();

		Message->ScreenShotName = Data.ScreenshotPath;
		Message->ScreenImage = CompressedBitmap;
		Message->FrameTrace = CapturedFrameTrace;
		Message->Metadata = Metadata;

		UE_LOGF(LogAutomationWorker, Log, "Sending screenshot %ls to %ls", *Message->ScreenShotName, *TestRequesterAddress.ToString());

		ActiveScreenshotComparisonId = Metadata.Id;

		SendMessage(Message, Message->StaticStruct(), TestRequesterAddress);
	}
	else
	{
		//Save locally
		const bool bTree = true;

		FString LocalFile = AutomationCommon::GetLocalPathForScreenshot(Data.ScreenshotPath);
		FString LocalTraceFile = FPaths::ChangeExtension(LocalFile, TEXT(".rdc"));
		FString DirectoryPath = FPaths::GetPath(LocalFile);

		UE_LOGF(LogAutomationWorker, Log, "Saving screenshot %ls as %ls",*Data.ScreenshotPath, *LocalFile);

		if (!IFileManager::Get().MakeDirectory(*DirectoryPath, bTree))
		{
			UE_LOGF(LogAutomationWorker, Error, "Failed to create directory %ls for incoming screenshot", *DirectoryPath);
			return;
		}

		if (!FFileHelper::SaveArrayToFile(CompressedBitmap, *LocalFile))
		{
			uint32 WriteErrorCode = FPlatformMisc::GetLastError();
			TCHAR WriteErrorBuffer[2048];
			FPlatformMisc::GetSystemErrorMessage(WriteErrorBuffer, 2048, WriteErrorCode);
			UE_LOGF(LogAutomationWorker, Warning, "Fail to save screenshot to %ls. WriteError: %u (%ls)", *LocalFile, WriteErrorCode, WriteErrorBuffer);
			return;
		}

		if (CapturedFrameTrace.Num() > 0)
		{
			if (!FFileHelper::SaveArrayToFile(CapturedFrameTrace, *LocalTraceFile))
			{
				uint32 WriteErrorCode = FPlatformMisc::GetLastError();
				TCHAR WriteErrorBuffer[2048];
				FPlatformMisc::GetSystemErrorMessage(WriteErrorBuffer, 2048, WriteErrorCode);
				UE_LOGF(LogAutomationWorker, Warning, "Failed to save frame trace to %ls. WriteError: %u (%ls)", *LocalTraceFile, WriteErrorCode, WriteErrorBuffer);
			}
		}

		FString Json;
		if ( FJsonObjectConverter::UStructToJsonObjectString(Metadata, Json) )
		{
			FString MetadataPath = FPaths::ChangeExtension(LocalFile, TEXT("json"));
			FFileHelper::SaveStringToFile(Json, *MetadataPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
		}
	}
#endif // WITH_AUTOMATION_TESTS
}
#endif

TSet<FName> GetRHIForAutomation()
{
	FString RHI = FApp::GetGraphicsRHI();
#if WITH_ENGINE
	FString FeatureLevel = LexToString(GMaxRHIFeatureLevel);
#else
	FString FeatureLevel = "N/A";
#endif

	if (RHI.IsEmpty())
	{
		RHI = FParse::Param(FCommandLine::Get(), TEXT("nullrhi"))? LexToString(ETEST_RHI_Options::Null) : TEXT("N/A");
	}
	else
	{
		// Remove any extra information in () from RHI string
		int Pos;
		if (RHI.FindChar(*TEXT("("), Pos))
		{
			RHI = RHI.Left(Pos).TrimEnd();
		}
	}

	return TSet<FName> {FName(RHI), FName(FeatureLevel)};
}

bool FAutomationWorkerModule::IsTestExcluded(const FString& InTestToRun, FString* OutReason, bool* OutWarn) const
{
	FName SkipReason;
	UAutomationTestExcludelist* Excludelist = UAutomationTestExcludelist::Get();
	check(nullptr != Excludelist);
	
	static const TSet<FName> RHI = GetRHIForAutomation();
	static const FName Platform = FPlatformProperties::IniPlatformName();
	if (Excludelist->IsTestExcluded(InTestToRun, Platform, RHI, &SkipReason, OutWarn))
	{
		if (OutReason)
		{
			(*OutReason) = (SkipReason.IsNone() ? TEXT("unknown reason") : SkipReason.ToString());
			(*OutReason) += TEXT(" [config]");

			if (const FAutomationTestExcludelistEntry* Entry = Excludelist->GetExcludeTestEntry(InTestToRun))
			{
				if (!Entry->Platforms.IsEmpty())
				{
					(*OutReason) += TEXT(" [");
					(*OutReason) += Platform.ToString();
					(*OutReason) += TEXT("]");
				}

				FString Filename = Excludelist->GetConfigFilenameForEntry(*Entry, Platform);

				if (!Filename.IsEmpty())
				{
					Filename = FPaths::ConvertRelativePathToFull(Filename);
					FPaths::MakePlatformFilename(Filename);
					*OutReason += TEXT(" [");
					*OutReason += Filename;
					// Using of line number 1 as a default value to get it working as a hyperlink
					*OutReason += TEXT("(1)]");
				}
			}
		}

		return true;
	}

	return false;
}

void FAutomationWorkerModule::HandleRunTestsMessage( const FAutomationWorkerRunTests& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	UE_LOGF(LogAutomationWorker, Log, "Received RunTests %ls from %ls", *Message.BeautifiedTestName, *Context->GetSender().ToString());

	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Worker"));
	
	if (TestRequesterAddress.IsValid() && !TestName.IsEmpty())
	{
		if (TestRequesterAddress == Context->GetSender())
		{
			UE_LOGF(LogAutomationWorker, Log, "Worker is already running test '%ls' from %ls. Request is ignored.", *BeautifiedTestName, *TestRequesterAddress.ToString());
			return;
		}

		FString LogMessage = FString::Format(TEXT("Worker is already running test '{0}' from {1}. '{2}' won't be run."), 
			{ *BeautifiedTestName, *TestRequesterAddress.ToString(), *Message.BeautifiedTestName });
		UE_LOGF(LogAutomationWorker, Warning, "%ls", *LogMessage);

		// Let the sender know it won't happen
		FAutomationWorkerRunTestsReply* OutMessage = FMessageEndpoint::MakeMessage<FAutomationWorkerRunTestsReply>();
		OutMessage->TestName = Message.TestName;
		OutMessage->ExecutionCount = Message.ExecutionCount;
		OutMessage->State = EAutomationState::Skipped;
		OutMessage->Entries.Add(FAutomationExecutionEntry(FAutomationEvent(EAutomationEventType::Error, LogMessage)));
		OutMessage->ErrorTotal = 1;
		SendMessage(OutMessage, OutMessage->StaticStruct(), Context->GetSender());

		return;
	}

	// Do we need to skip the test
	FString SkipReason;
	bool bWarn(false);
	FAutomationTestFramework& AutomationTestFramework = FAutomationTestFramework::Get();
	if (!AutomationTestFramework.CanRunTestInEnvironment(Message.TestName, &SkipReason, &bWarn)
		|| IsTestExcluded(Message.FullTestPath, &SkipReason, &bWarn))
	{
		FString SkippingMessage = FString::Format(TEXT("Test Skipped. Name={{0}} Reason={{1}} Path={{2}}"),
			{ *Message.BeautifiedTestName, *SkipReason, *Message.FullTestPath });

		if (bWarn)
		{
			UE_LOGF(LogAutomationWorker, Warning, "%ls", *SkippingMessage);
		}
		else
		{
			UE_LOGF(LogAutomationWorker, Display, "%ls", *SkippingMessage);
		}

		FAutomationWorkerRunTestsReply* OutMessage = FMessageEndpoint::MakeMessage<FAutomationWorkerRunTestsReply>();
		OutMessage->TestName = Message.TestName;
		OutMessage->ExecutionCount = Message.ExecutionCount;
		OutMessage->State = EAutomationState::Skipped;
		OutMessage->Entries.Add(FAutomationExecutionEntry(FAutomationEvent(EAutomationEventType::Info, FString::Printf(TEXT("Skipping test: %s"), *SkipReason))));
		SendMessage(OutMessage, OutMessage->StaticStruct(), Context->GetSender());

		return;
	}

	ExecutionCount = Message.ExecutionCount;
	TestName = Message.TestName;
	BeautifiedTestName = Message.BeautifiedTestName;
	FullTestPath = Message.FullTestPath;
	bSendAnalytics = Message.bSendAnalytics;
	bPruneLogsOnSuccess = Message.bPruneLogsOnSuccess;
	TestRequesterAddress = Context->GetSender();
	ActiveScreenshotComparisonId = FGuid();

	// Always allow the first network command to execute
	bExecuteNextNetworkCommand = true;

	// We are not executing network command sub-commands right now
	bExecutingNetworkCommandResults = false;

	// Track active section
	TriggerSectionNotifications();

	FAutomationTestFramework::Get().StartTestByName(Message.TestName, Message.RoleIndex, Message.FullTestPath);
}

void FAutomationWorkerModule::TriggerSectionNotifications()
{
	if (FullTestPath.IsEmpty())
	{
		// Unwind
		if (!ActiveSection.IsEmpty())
		{
			if (FAutomationTestFramework::Get().IsAnyOnLeavingTestSectionBound())
			{
				FAutomationTestFramework::Get().TriggerOnLeavingTestSection(ActiveSection);
				int32 Pos;
				while (ActiveSection.FindLastChar('.', Pos))
				{
					ActiveSection.LeftInline(Pos);
					FAutomationTestFramework::Get().TriggerOnLeavingTestSection(ActiveSection);
				}
			}
			ActiveSection.Empty();
		}
		return;
	}

	if (!FAutomationTestFramework::Get().IsAnyOnEnteringTestSectionBound()
		&& !FAutomationTestFramework::Get().IsAnyOnLeavingTestSectionBound())
	{
		return;
	}

	// Gather nesting sections
	TArray<FString> NestingSections;
	{
		int32 Pos;
		FString Section = FullTestPath;
		while (Section.FindLastChar('.', Pos))
		{
			Section.LeftInline(Pos);
			if (ActiveSection.IsEmpty() || !ActiveSection.StartsWith(Section))
			{
				NestingSections.Add(Section);
			}
			else
			{
				break;
			}
		}
	}

	if (NestingSections.Num() > 0)
	{
		// Notify leaving sections
		if (!ActiveSection.IsEmpty() && FAutomationTestFramework::Get().IsAnyOnLeavingTestSectionBound())
		{
			const FString TargetSection = NestingSections[0];
			int32 Pos;
			while (!TargetSection.StartsWith(ActiveSection) && ActiveSection.FindLastChar('.', Pos))
			{
				FAutomationTestFramework::Get().TriggerOnLeavingTestSection(ActiveSection);
				ActiveSection.LeftInline(Pos);
			}
		}

		// Notify entering sections
		ActiveSection = NestingSections[0];
		if (FAutomationTestFramework::Get().IsAnyOnEnteringTestSectionBound())
		{
			Algo::Reverse(NestingSections);
			for (const FString& Section : NestingSections)
			{
				FAutomationTestFramework::Get().TriggerOnEnteringTestSection(Section);
			}
		}
	}
}

void FAutomationWorkerModule::HandleStopTestsMessage(const FAutomationWorkerStopTests& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationWorker, Log, "Received StopTests from %ls", *Context->GetSender().ToString());

	if (GIsAutomationTesting)
	{
		FAutomationTestFramework::Get().DequeueAllCommands();
	}
	ReportTestComplete();
}


//dispatches analytics events to the data collector
void FAutomationWorkerModule::SendAnalyticsEvents(TArray<FString>& InAnalyticsItems)
{
	for (int32 i = 0; i < InAnalyticsItems.Num(); ++i)
	{
		FString EventString = InAnalyticsItems[i];
		if( EventString.EndsWith( TEXT( ",PERF" ) ) )
		{
			// Chop the ",PERF" off the end
			EventString.LeftInline( EventString.Len() - 5, EAllowShrinking::No);

			FAutomationPerformanceSnapshot PerfSnapshot;
			PerfSnapshot.FromCommaDelimitedString( EventString );
			
			RecordPerformanceAnalytics( PerfSnapshot );
		}
	}
}

void FAutomationWorkerModule::HandleTelemetryData(const FString& StorageName, const FString& InTestName, const TArray<FAutomationTelemetryData>& InItems)
{
	FAutomationWorkerTelemetryData* Message = FMessageEndpoint::MakeMessage<FAutomationWorkerTelemetryData>();

	Message->Storage = StorageName;
	Message->Platform = FPlatformProperties::PlatformName();
	Message->Configuration = LexToString(FApp::GetBuildConfiguration());
	Message->TestName = InTestName;
	for (const FAutomationTelemetryData& Item : InItems)
	{
		Message->Items.Add(FAutomationWorkerTelemetryItem(Item));
	}

	UE_LOGF(LogAutomationWorker, Log, "Sending Telemetry Data for %ls", *Message->TestName);

	SendMessage(Message, Message->StaticStruct(), TestRequesterAddress);
}

void FAutomationWorkerModule::RecordPerformanceAnalytics( const FAutomationPerformanceSnapshot& PerfSnapshot )
{
	// @todo: Pass in additional performance capture data from incoming FAutomationPerformanceSnapshot!

	FAutomationAnalytics::FireEvent_FPSCapture(PerfSnapshot);
}

#undef LOCTEXT_NAMESPACE
