// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IMessageContext.h"
#include "Misc/AutomationTest.h"

#include "IAutomationWorkerModule.h"

class FMessageEndpoint;

struct FAutomationWorkerFindWorkers;
struct FAutomationWorkerImageComparisonResults;
struct FAutomationWorkerTestDataResponse;
struct FAutomationWorkerPerformanceDataResponse;
struct FAutomationWorkerNextNetworkCommandReply;
struct FAutomationWorkerPing;
struct FAutomationWorkerRequestTests;
struct FAutomationWorkerStartTestSession;
struct FAutomationWorkerStopTestSession;
struct FAutomationWorkerRunTests;
struct FMessageAddress;
struct FAutomationWorkerMessageBase;


/**
 * Implements the Automation Worker module.
 */
class FAutomationWorkerModule
	: public IAutomationWorkerModule
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;

public:

	//~ IAutomationWorkerModule interface

	virtual void Tick() override;

protected:

	/** Execute all latent commands and when complete send results message back to the automation controller. */
	bool ExecuteLatentCommands();

	/** Execute all network commands and when complete send results message back to the automation controller. */
	bool ExecuteNetworkCommands();

	/** Initializes the automation worker. */
	void Initialize();

	/** Network phase is complete (if there were any network commands).  Send ping back to the controller. */
	void ReportNetworkCommandComplete();

	/** Test is complete. Send results back to controller. */
	void ReportTestComplete();

	/** 
	 * Send a list of all the tests supported by the worker
	 *
	 * @param ControllerAddress The message address of the controller that requested the tests.
	 */
	void SendTests(const FMessageAddress& ControllerAddress);

	/**
	 * Send a message in an unified way with passing correct Instance Id into the message we are going to send.
	 * 
	 * @param Message The message to be sent.
	 * @param TypeInfo The type information about the message to be sent.
	 * @param ControllerAddress The message address of the receiver.
	 */
	void SendMessage(FAutomationWorkerMessageBase* Message, UScriptStruct* TypeInfo, const FMessageAddress& ControllerAddress);

private:

	/** Handles FAutomationWorkerFindWorkers messages. */
	void HandleFindWorkersMessage(const FAutomationWorkerFindWorkers& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Deferred handler for sending "find worker" response in case the asset registry isn't loaded yet. */
	void SendWorkerFound(const FMessageAddress& ControllerAddress);

	/** Handles message endpoint shutdowns. */
	void HandleMessageEndpointShutdown();

	/** Handles FAutomationWorkerNextNetworkCommandReply messages. */
	void HandleNextNetworkCommandReplyMessage(const FAutomationWorkerNextNetworkCommandReply& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerPing messages. */
	void HandlePingMessage(const FAutomationWorkerPing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerStartTestSession messages. */
	void HandleStartTestSession(const FAutomationWorkerStartTestSession& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerStopTestSession messages. */
	void HandleStopTestSession(const FAutomationWorkerStopTestSession& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerRequestTests messages. */
	void HandleRequestTestsMessage(const FAutomationWorkerRequestTests& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerRunTests messages. */
	void HandleRunTestsMessage(const FAutomationWorkerRunTests& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerStopTests messages. */
	void HandleStopTestsMessage(const struct FAutomationWorkerStopTests& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerImageComparisonResults messages. */
	void HandleScreenShotCompared(const FAutomationWorkerImageComparisonResults& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerTestDataResponse messages. */
	void HandleTestDataRetrieved(const FAutomationWorkerTestDataResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerPerformanceDataResponse messages. */
	void HandlePerformanceDataRetrieved(const FAutomationWorkerPerformanceDataResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Dispatches telemetry data to data collector */
	void HandleTelemetryData(const FString& StorageName, const FString& InTestName, const TArray<FAutomationTelemetryData>& InItems);

	/** Handles FAutomationTestFramework PreTestingEvents. */
	void HandlePreTestingEvent();

	/** Handles FAutomationTestFramework PostTestingEvents. */
	void HandlePostTestingEvent();

#if WITH_ENGINE
	/** Invoked when we have screen shot to send. */
	void HandleScreenShotCapturedWithName(const TArray<FColor>& RawImageData, const FAutomationScreenshotData& Data);

	/** Invoked when we have screen shot and frame trace to send. */
	void HandleScreenShotAndTraceCapturedWithName(const TArray<FColor>& RawImageData, const TArray<uint8>& CapturedFrameTrace, const FAutomationScreenshotData& Data);

	/** Invoked when we have a screenshot comparison result to send. */
	void HandleScreenShotComparisonReport(const FAutomationScreenshotCompareResults& Results);
#endif

	/** Dispatches analytics events to the data collector. */
	void SendAnalyticsEvents(TArray<FString>& InAnalyticsItems);

	/** Helper for Performance Capture Analytics. */
	void RecordPerformanceAnalytics(const FAutomationPerformanceSnapshot& PerfSnapshot);

	/** Checks whether the test is in exclusion list. */
	bool IsTestExcluded(const FString& InTestToRun, FString* OutReason, bool* OutWarn) const;

	/** Trigger Notifications when entering and leaving section */
	void TriggerSectionNotifications();

private:

	/** The collection of test data we are to send to a controller. */
	TArray<FAutomationTestInfo> TestInfo;

private:

	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Message address of the controller sending the test request. */
	FMessageAddress TestRequesterAddress;

	/** Identifier for the controller to know if the results should be discarded or not. */
	uint32 ExecutionCount;

	/** Execute one of the tests by request of the controller. */
	FString TestName;

	/** Beautified name of the test */
	FString BeautifiedTestName;

	/** Full path of the test */
	FString FullTestPath;

	/** Whether to send analytics events to the backend - sent from controller */
	bool bSendAnalytics;

	/** Whether to prune log events from test report on success */
	bool bPruneLogsOnSuccess;

	/** Whether the controller has requested that the network command should execute */
	bool bExecuteNextNetworkCommand;

	/** Whether we are processing sub-commands of a network command */
	bool bExecutingNetworkCommandResults;

	/** Delegate to fire when the test is complete */
	FStopTestEvent StopTestEvent;

	/** Tag for the device on which the worker is running */
	FString DeviceTag;

	/** Tracking of active section */
	FString ActiveSection;

	/** Tracking of the active screenshot comparison ID **/
	FGuid ActiveScreenshotComparisonId;
};
