// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Wraps up useful performance capture data for automation tests.  These are usually triggered at
 * specific intervals along a timeline inside an automation-specific cinematic asset.
 */
struct FAutomationPerformanceSnapshot
{
	FString Changelist;
	FString BuildConfiguration;
	FString MapName;
	FString SequenceName;
	FString AverageFPS;
	FString AverageFrameTime;
	FString AverageGameThreadTime;
	FString AverageRenderThreadTime;
	FString AverageGPUTime;
	FString PercentOfFramesAtLeast30FPS;
	FString PercentOfFramesAtLeast60FPS;


	/** @return	Serializes this structure into a simple comma-delimited string */
	FString ToCommaDelimetedString() const
	{
		return
			Changelist + TEXT( "," ) +
			BuildConfiguration + TEXT( "," ) +
			MapName + TEXT( "," ) +
			SequenceName + TEXT( "," ) +
			AverageFPS + TEXT( "," ) +
			AverageFrameTime + TEXT( "," ) +
			AverageGameThreadTime + TEXT( "," ) +
			AverageRenderThreadTime + TEXT( "," ) +
			AverageGPUTime + TEXT( "," ) +
			PercentOfFramesAtLeast30FPS + TEXT( "," ) +
			PercentOfFramesAtLeast60FPS;
	}

	/** Deserializes from a comma-delimited string created with MakeCommaDeliminedString */
	void FromCommaDelimitedString( const FString& CommaDelimitedString )
	{
		TArray<FString> Values;
		const bool bCullEmpty = false;
		CommaDelimitedString.ParseIntoArray( Values, TEXT( "," ), bCullEmpty );

		int32 NextValue = 0;
		Changelist = Values[ NextValue++ ];
		BuildConfiguration = Values[ NextValue++ ];
		MapName = Values[ NextValue++ ];
		SequenceName = Values[ NextValue++ ];
		AverageFPS = Values[ NextValue++ ];
		AverageFrameTime = Values[ NextValue++ ];
		AverageGameThreadTime = Values[ NextValue++ ];
		AverageRenderThreadTime = Values[ NextValue++ ];
		AverageGPUTime = Values[ NextValue++ ];
		PercentOfFramesAtLeast30FPS = Values[ NextValue++ ];
		PercentOfFramesAtLeast60FPS = Values[ NextValue++ ];
	}
};


/**
 * Interface for AutomationWorker modules.
 */
class IAutomationWorkerModule
	: public IModuleInterface
{
public:
	/** Called whenever a test is stopped */
	DECLARE_DELEGATE_ThreeParams(FStopTestEvent, bool, FString, class FAutomationTestExecutionInfo const &);

	/**
	 * Ticks the automation worker module.
	 */
	virtual void Tick( ) = 0;
	/**
	 * Start a test
	 * @param	InTestToRun			Name of the test that should be run
	 * @param	InRoleIndex			Identifier for which worker in this group that should execute a command
	 * @param	InStopTestEvent		Delegate to fire when the command is finished
	 */
	//virtual void RunTest(const FString& InTestToRun, const int32 InRoleIndex, FStopTestEvent const& InStopTestEvent) = 0;


protected:

	IAutomationWorkerModule() { }
};
